// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "MyPluginObject.generated.h"


/**
 * Example UStruct declared in a plugin module
 */
USTRUCT()
struct FMyPluginStruct
{
	GENERATED_USTRUCT_BODY()
 
	UPROPERTY()
	FString TestString;
};
 

/**
 * Example of declaring a UObject in a plugin module
 */
UCLASS()
class UMyPluginObject : public UObject
{
	GENERATED_UCLASS_BODY()

public:

private:

	UPROPERTY()
	FMyPluginStruct MyStruct;

};


