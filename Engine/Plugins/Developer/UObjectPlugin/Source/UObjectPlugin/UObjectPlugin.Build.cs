// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class UObjectPlugin : ModuleRules
	{
		public UObjectPlugin(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicIncludePaths.AddRange(
				new string[] {
					// ... add public include paths required here ...
				}
				);

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					// ... add other public dependencies that you statically link with here ...
				}
				);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					// ... add private dependencies that you statically link with here ...
				}
				);

			DynamicallyLoadedModuleNames.AddRange(
				new string[]
				{
					// ... add any modules that your module loads dynamically here ...
				}
				);
		}
	}
}
