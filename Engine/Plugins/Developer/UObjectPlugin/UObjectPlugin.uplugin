{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "UObject Example Plugin",
	"Description" : "An example of a plugin which declares its own UObject type.  This can be used as a starting point when creating your own plugin.",
	"Category" : "Examples",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "UObjectPlugin",
			"Type" : "Runtime",
			"LoadingPhase" : "Default"
		}
	]
}